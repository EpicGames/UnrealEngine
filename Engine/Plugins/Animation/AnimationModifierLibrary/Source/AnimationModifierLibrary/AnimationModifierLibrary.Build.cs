// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AnimationModifierLibrary : ModuleRules
	{
		public AnimationModifierLibrary(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"AnimationModifiers",
					"AnimationBlueprintLibrary"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
				}
			);

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
