// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationModifierLibrary.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AnimationModifierLibraryModule"

void FAnimationModifierLibraryModule::StartupModule()
{
}

void FAnimationModifierLibraryModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAnimationModifierLibraryModule, AnimationModifierLibrary)
