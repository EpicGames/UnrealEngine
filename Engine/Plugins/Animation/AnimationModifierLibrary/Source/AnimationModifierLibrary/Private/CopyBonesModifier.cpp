// Copyright Epic Games, Inc. All Rights Reserved.

#include "CopyBonesModifier.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "EngineLogs.h"
#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CopyBonesModifier)

#define LOCTEXT_NAMESPACE "CopyBonesModifier"

UCopyBonesModifier::UCopyBonesModifier()
	:Super()
{
}

void UCopyBonesModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "CopyBonesModifier failed. Reason: Invalid Animation");
		return;
	}

	IAnimationDataController& Controller = Animation->GetController();
	const IAnimationDataModel* Model = Animation->GetDataModel();

	if (Model == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "CopyBonesModifier failed. Reason: Invalid Data Model. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	// Helper structure to store data for the bones we are going to modify
	struct FCopyBoneData
	{
		FName SourceBoneName = NAME_None;
		FName TargetBoneName = NAME_None;
		int32 SourceBoneIdx = INDEX_NONE;
		int32 TargetBoneIdx = INDEX_NONE;
		FCopyBoneData(const FName& InSourceBoneName, const FName& InTargetBoneName, int32 InSourceBoneIdx, int32 InTargetBoneIdx)
			: SourceBoneName(InSourceBoneName), TargetBoneName(InTargetBoneName), SourceBoneIdx(InSourceBoneIdx), TargetBoneIdx(InTargetBoneIdx) {}
	};

	const USkeleton* Skeleton = Animation->GetSkeleton();
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	// Validate input
	TArray<FCopyBoneData> CopyBoneDataContainer;
	CopyBoneDataContainer.Reserve(BonePairs.Num());
	for (const FBoneReferencePair& Pair : BonePairs)
	{
		const int32 SourceBoneIdx = RefSkeleton.FindBoneIndex(Pair.SourceBone.BoneName);
		if (SourceBoneIdx == INDEX_NONE)
		{
			continue;
		}

		const int32 TargetBoneIdx = RefSkeleton.FindBoneIndex(Pair.TargetBone.BoneName);
		if (TargetBoneIdx == INDEX_NONE)
		{
			continue;
		}

		CopyBoneDataContainer.Add(FCopyBoneData(Pair.SourceBone.BoneName, Pair.TargetBone.BoneName, SourceBoneIdx, TargetBoneIdx));
	}
	
	// Sort bones to modify so we always modify parents first
	CopyBoneDataContainer.Sort([](const FCopyBoneData& A, const FCopyBoneData& B) { return A.TargetBoneIdx < B.TargetBoneIdx; });

	// Temporally set ForceRootLock to true so we get the correct transforms regardless of the root motion configuration in the animation
	TGuardValue<bool> ForceRootLockGuard(Animation->bForceRootLock, true);

	// Start editing animation data
	constexpr bool bShouldTransact = false;
	Controller.OpenBracket(LOCTEXT("CopyBonesModifier_Bracket", "Updating bones"), bShouldTransact);

	// Get the transform of all the source bones in the desired space
	const int32 NumKeys = Model->GetNumberOfKeys();
	const FInt32Range KeyRangeToSet(0, NumKeys);
	struct FPerBoneData
	{
		TArray<FVector> PositionalKeys;
		TArray<FQuat> RotationalKeys;
		TArray<FVector> ScalingKeys;
	};

	TArray<FPerBoneData> PerBoneDataContainer;

	// Pre-allocate bone arrays for each copy
	const int32 NumBoneCopies = CopyBoneDataContainer.Num();
	PerBoneDataContainer.SetNum(CopyBoneDataContainer.Num());
	for (int32 Index = 0; Index < NumBoneCopies; ++Index)
	{
		PerBoneDataContainer[Index].PositionalKeys.Reserve(NumKeys);
		PerBoneDataContainer[Index].RotationalKeys.Reserve(NumKeys);
		PerBoneDataContainer[Index].ScalingKeys.Reserve(NumKeys);
	}

	FScopedSlowTask SlowTask(static_cast<float>(CopyBoneDataContainer.Num() + NumKeys));
	for (int32 AnimKey = 0; AnimKey < NumKeys; AnimKey++)
	{
		FAnimPose AnimPose;
		UAnimPoseExtensions::GetAnimPoseAtFrame(Animation, AnimKey, FAnimPoseEvaluationOptions(), AnimPose);

		SlowTask.EnterProgressFrame();
		for (int32 Index = 0; Index < NumBoneCopies; ++Index)
		{
			FPerBoneData& PerBoneData = PerBoneDataContainer[Index];
			FCopyBoneData& Data = CopyBoneDataContainer[Index];
			
			// Make a copy of the pose to deal with potential parent-chain issues
			FAnimPose AnimPoseCopy = AnimPose;
			FTransform BonePose = UAnimPoseExtensions::GetBonePose(AnimPoseCopy, Data.SourceBoneName, BonePoseSpace);
			
			// UAnimDataController::UpdateBoneTrackKeys expects local transforms so we need to convert the source transforms to target bone local transforms first. 
			UAnimPoseExtensions::SetBonePose(AnimPoseCopy, BonePose, Data.TargetBoneName, BonePoseSpace);
			FTransform BonePoseTargetLocal = UAnimPoseExtensions::GetBonePose(AnimPoseCopy, Data.TargetBoneName, EAnimPoseSpaces::Local);

			PerBoneData.PositionalKeys.Add(BonePoseTargetLocal.GetLocation());
			PerBoneData.RotationalKeys.Add(BonePoseTargetLocal.GetRotation());
			PerBoneData.ScalingKeys.Add(BonePoseTargetLocal.GetScale3D());
		}
	}

	
	for (int32 Index = 0; Index < NumBoneCopies; ++Index)
	{
		SlowTask.EnterProgressFrame();
		FCopyBoneData& Data = CopyBoneDataContainer[Index];
		FPerBoneData& PerBoneData = PerBoneDataContainer[Index];

		// If there are no pre-existing curves for the target bone we have to add them before being able to set the keys
		if(!Model->IsValidBoneTrackName(Data.TargetBoneName))
		{
			Controller.AddBoneCurve(Data.TargetBoneName, bShouldTransact);
		}
		
		Controller.SetBoneTrackKeys(Data.TargetBoneName, PerBoneData.PositionalKeys, PerBoneData.RotationalKeys, PerBoneData.ScalingKeys, bShouldTransact);
	}

	// Done editing animation data
	Controller.CloseBracket(bShouldTransact);
}

void UCopyBonesModifier::OnRevert_Implementation(UAnimSequence* Animation)
{
	// This AnimModifier doesn't support Revert operation
}

#undef LOCTEXT_NAMESPACE
