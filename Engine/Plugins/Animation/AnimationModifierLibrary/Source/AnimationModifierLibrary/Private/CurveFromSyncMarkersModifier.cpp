// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveFromSyncMarkersModifier.h"
#include "Animation/AnimSequence.h"
#include "AnimationBlueprintLibrary.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveFromSyncMarkersModifier)

#define LOCTEXT_NAMESPACE "CurveFromSyncMarkersModifier"

void UCurveFromSyncMarkersModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "CurveFromSyncMarkersModifier failed. Reason: Invalid Animation");
		return;
	}

	if (CurveName.IsNone())
	{
		UE_LOGF(LogAnimation, Error, "CurveFromSyncMarkersModifier failed. Reason: Invalid CurveName");
		return;
	}

	if (SyncMarkerValues.IsEmpty())
	{
		UE_LOGF(LogAnimation, Warning, "CurveFromSyncMarkersModifier: since SyncMarkerValues are empty, no curve will be generated");
		return;
	}

	// Build a lookup from sync marker name to float value
	TMap<FName, float> MarkerValueMap;
	for (const FSyncMarkerCurveEntry& Entry : SyncMarkerValues)
	{
		if (!Entry.SyncMarkerName.IsNone())
		{
			MarkerValueMap.Add(Entry.SyncMarkerName, Entry.Value);
		}
	}

	// Collect curve keys from authored sync markers whose names match the input list
	TArray<FRichCurveKey> CurveKeys;
	for (const FAnimSyncMarker& Marker : Animation->AuthoredSyncMarkers)
	{
		if (const float* ValuePtr = MarkerValueMap.Find(Marker.MarkerName))
		{
			FRichCurveKey& Key = CurveKeys.AddDefaulted_GetRef();
			Key.Time = Marker.Time;
			Key.Value = *ValuePtr;
		}
	}

	if (CurveKeys.IsEmpty())
	{
		UE_LOGF(LogAnimation, Warning, "CurveFromSyncMarkersModifier: No matching sync markers found. Curve will not be generated");
		return;
	}

	if (UAnimationBlueprintLibrary::DoesCurveExist(Animation, CurveName, ERawCurveTrackTypes::RCT_Float))
	{
		UE_LOGF(LogAnimation, Warning, "CurveFromSyncMarkersModifier: Curve '%ls' already exists on Animation '%ls'. Curve will not be generated.", *CurveName.ToString(), *Animation->GetName());
		return;
	}

	// AuthoredSyncMarkers should already be in time order, but sort defensively
	CurveKeys.Sort([](const FRichCurveKey& A, const FRichCurveKey& B) { return A.Time < B.Time; });

	// For looping animations, add boundary keys at time 0 and animation end so the curve
	// is continuous across the wrap seam. The boundary value is linearly interpolated across
	// the segment from the last key forward (past the animation end) to the first key.
	if (Animation->bLoop)
	{
		const float AnimLength = Animation->GetPlayLength();
		const FRichCurveKey& FirstKey = CurveKeys[0];
		const FRichCurveKey& LastKey  = CurveKeys.Last();

		// Segment spans from LastKey.Time to FirstKey.Time + AnimLength (wrapping around)
		const float SegmentDuration = (FirstKey.Time + AnimLength) - LastKey.Time;
		const float BoundaryValue = (SegmentDuration > UE_KINDA_SMALL_NUMBER)
			? FMath::Lerp(LastKey.Value, FirstKey.Value, (AnimLength - LastKey.Time) / SegmentDuration)
			: LastKey.Value;

		// Prepend at time 0 only when no marker already sits there
		if (FirstKey.Time > UE_KINDA_SMALL_NUMBER)
		{
			FRichCurveKey StartKey;
			StartKey.Time  = 0.f;
			StartKey.Value = BoundaryValue;
			CurveKeys.Insert(StartKey, 0);
		}

		// Append at animation end only when no marker already sits there
		// (using CurveKeys.Last() instead of LastKey since CurveKeys could have been reallocated)
		if ((AnimLength - CurveKeys.Last().Time) > UE_KINDA_SMALL_NUMBER)
		{
			FRichCurveKey EndKey;
			EndKey.Time  = AnimLength;
			EndKey.Value = BoundaryValue;
			CurveKeys.Add(EndKey);
		}
	}

	IAnimationDataController& Controller = Animation->GetController();
	Controller.OpenBracket(LOCTEXT("CurveFromSyncMarkers_Bracket", "Adding Curve From Sync Markers"));

	const FAnimationCurveIdentifier CurveId = UAnimationCurveIdentifierExtensions::GetCurveIdentifier(
		Animation->GetSkeleton(), CurveName, ERawCurveTrackTypes::RCT_Float);

	if (Controller.AddCurve(CurveId))
	{
		Controller.SetCurveKeys(CurveId, CurveKeys);
	}

	Controller.CloseBracket();
}

void UCurveFromSyncMarkersModifier::OnRevert_Implementation(UAnimSequence* Animation)
{
	if (bRemoveCurveOnRevert)
	{
		const bool bRemoveNameFromSkeleton = false;
		UAnimationBlueprintLibrary::RemoveCurve(Animation, CurveName, bRemoveNameFromSkeleton);
	}
}

#undef LOCTEXT_NAMESPACE
