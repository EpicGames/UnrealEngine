// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#define ENABLE_ENCODE_ROOT_BONE_MODIFIER_TEST 0

#include "EncodeRootBoneModifier.h"
#include "Animation/AnimSequence.h"
#include "Misc/ScopedSlowTask.h"
#include "PoseExtractorUtilityLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EncodeRootBoneModifier)

#define LOCTEXT_NAMESPACE "EncodeRootBoneModifier"

void UEncodeRootBoneModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	FPoseExtractorUtilityLibrary Extractor(Animation);

	float TotalWeightToComputeRootPosition = 0.f;
	for (const FEncodeRootBoneWeightedBone& WeightedBone : WeightedBoneToComputeRootPosition)
	{
		if (!Extractor.AddBone(WeightedBone.Bone))
		{
			return;
		}
		TotalWeightToComputeRootPosition += WeightedBone.Weight;
	}

	float TotalWeightToComputeRootOrientation = 0.f;
	for (const FEncodeRootBoneWeightedBone& WeightedBone : WeightedBoneToComputeRootOrientation)
	{
		if (!Extractor.AddBone(WeightedBone.Bone))
		{
			return;
		}
		TotalWeightToComputeRootOrientation += WeightedBone.Weight;
	}

	// Collect tracks for the bones we are going to modify (root and direct children of the root)
	TArray<FName> RootBoneChildBones;
	TArray<int32> ChildBoneIndices;
	Animation->GetSkeleton()->GetReferenceSkeleton().GetDirectChildBones(0, ChildBoneIndices);
	for (const int32 ChildBoneIndex : ChildBoneIndices)
	{
		const FName ChildBoneName = Animation->GetSkeleton()->GetReferenceSkeleton().GetBoneName(ChildBoneIndex);
		if (Animation->GetDataModel()->IsValidBoneTrackName(ChildBoneName))
		{
			RootBoneChildBones.Add(ChildBoneName);
			if (!Extractor.AddBone(ChildBoneName, ChildBoneIndex))
			{
				return;
			}
		}
	}

	if (!Extractor.CacheGlobalBoneTransforms())
	{
		return;
	}

	// Start editing animation data
	const bool bShouldTransact = false;
	Animation->GetController().OpenBracket(LOCTEXT("EncodeRootBoneModifier_Bracket", "Encoding root bone"), bShouldTransact);
	// For each key in the animation
	const int32 NumAnimKeys = Animation->GetDataModel()->GetNumberOfKeys();
	FScopedSlowTask SlowTask(static_cast<float>(NumAnimKeys));
	SlowTask.MakeDialog();
	for (int32 AnimKey = 0; AnimKey < NumAnimKeys; AnimKey++)
	{
		SlowTask.EnterProgressFrame();
		const FInt32Range KeyRangeToSet(AnimKey, AnimKey + 1);

		const FTransform RootTransformOriginal = Extractor.GetGlobalBoneTransform(AnimKey, 0);
		FTransform RootTransformNew = RootTransformOriginal;
		if (TotalWeightToComputeRootPosition > UE_KINDA_SMALL_NUMBER)
		{
			FVector WeightedBoneTranslation = FVector::ZeroVector;
			for (const FEncodeRootBoneWeightedBone& WeightedBone : WeightedBoneToComputeRootPosition)
			{
				WeightedBoneTranslation += Extractor.GetGlobalBoneTransform(AnimKey, WeightedBone.Bone.BoneName).GetTranslation() * WeightedBone.Weight;
			}

			WeightedBoneTranslation /= TotalWeightToComputeRootPosition;
			WeightedBoneTranslation.Z = RootTransformNew.GetTranslation().Z;
			RootTransformNew.SetTranslation(WeightedBoneTranslation);
		}

		if (TotalWeightToComputeRootOrientation > UE_KINDA_SMALL_NUMBER)
		{
			FVector WeightedBoneHeading = FVector::ZeroVector;
			for (const FEncodeRootBoneWeightedBoneAxis& WeightedBoneAxis : WeightedBoneToComputeRootOrientation)
			{
				const FQuat GlobalBoneRotation = Extractor.GetGlobalBoneTransform(AnimKey, WeightedBoneAxis.Bone.BoneName).GetRotation();
				switch (WeightedBoneAxis.BoneAxis)
				{
				case EEncodeRootBoneAxis::X:
					WeightedBoneHeading += GlobalBoneRotation.GetAxisX() * WeightedBoneAxis.Weight;
					break;
				case EEncodeRootBoneAxis::Y:
					WeightedBoneHeading += GlobalBoneRotation.GetAxisY() * WeightedBoneAxis.Weight;
					break;
				case EEncodeRootBoneAxis::Z:
					WeightedBoneHeading += GlobalBoneRotation.GetAxisZ() * WeightedBoneAxis.Weight;
					break;
				}
			}

			const double Yaw = FMath::Atan2(-WeightedBoneHeading.X, WeightedBoneHeading.Y) * (180.f / UE_PI);
			const FRotator Rotator(0.f, Yaw, 0.f);
			RootTransformNew.SetRotation(Rotator.Quaternion());
		}

		Animation->GetController().UpdateBoneTrackKeys(Animation->GetSkeleton()->GetReferenceSkeleton().GetBoneName(0), KeyRangeToSet, { RootTransformNew.GetLocation() }, { RootTransformNew.GetRotation() }, { RootTransformNew.GetScale3D() }, bShouldTransact);

		// Now the mesh is facing the wrong axis. Update direct children of the root with the local space transform that puts them back to where they were originally
		for (const FName& ChildBoneName : RootBoneChildBones)
		{
			const FTransform ChildBoneNewTransform = Extractor.GetGlobalBoneTransform(AnimKey, ChildBoneName).GetRelativeTransform(RootTransformNew);
			Animation->GetController().UpdateBoneTrackKeys(ChildBoneName, KeyRangeToSet, { ChildBoneNewTransform.GetLocation() }, { ChildBoneNewTransform.GetRotation() }, { ChildBoneNewTransform.GetScale3D() }, bShouldTransact);
		}
	}

	// Done editing animation data
	Animation->GetController().CloseBracket(bShouldTransact);


#if ENABLE_ENCODE_ROOT_BONE_MODIFIER_TEST && DO_CHECK
	// testing
	FPoseExtractorUtilityLibrary TestExtractor(Animation);
	for (const FEncodeRootBoneWeightedBone& WeightedBone : WeightedBoneToComputeRootPosition)
	{
		check(TestExtractor.AddBone(WeightedBone.Bone));
	}
	for (const FEncodeRootBoneWeightedBone& WeightedBone : WeightedBoneToComputeRootOrientation)
	{
		check(TestExtractor.AddBone(WeightedBone.Bone));
	}
	for (const int32 ChildBoneIndex : ChildBoneIndices)
	{
		const FName ChildBoneName = Animation->GetSkeleton()->GetReferenceSkeleton().GetBoneName(ChildBoneIndex);
		if (Animation->GetDataModel()->IsValidBoneTrackName(ChildBoneName))
		{
			check(TestExtractor.AddBone(ChildBoneName, ChildBoneIndex));
		}
	}

	check(TestExtractor.CacheGlobalBoneTransforms());

	for (const FName& ChildBoneName : RootBoneChildBones)
	{
		for (int32 AnimKey = 0; AnimKey < NumAnimKeys; AnimKey++)
		{
			const FTransform ChildBoneTransform = TestExtractor.GetGlobalBoneTransform(AnimKey, ChildBoneName);
			const FTransform TestChildBoneTransform = TestExtractor.GetGlobalBoneTransform(AnimKey, ChildBoneName);

			static float EncodeRootBoneModifierTolerance = UE_KINDA_SMALL_NUMBER;
			check(ChildBoneTransform.Equals(TestChildBoneTransform, EncodeRootBoneModifierTolerance));
		}
	}

#endif // ENABLE_ENCODE_ROOT_BONE_MODIFIER_TEST && DO_CHECK
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
