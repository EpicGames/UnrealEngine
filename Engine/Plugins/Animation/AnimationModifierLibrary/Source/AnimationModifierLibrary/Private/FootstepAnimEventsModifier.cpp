// Copyright Epic Games, Inc. All Rights Reserved.


#if WITH_EDITOR

#include "FootstepAnimEventsModifier.h"
#include "Animation/AnimSequence.h"
#include "AnimationBlueprintLibrary.h"
#include "AnimPose.h"
#include "Animation/AnimNotifies/AnimNotify.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FootstepAnimEventsModifier)

#define LOCTEXT_NAMESPACE "FootstepAnimEventsModifierBase"

UFootstepAnimEventsModifier::UFootstepAnimEventsModifier() : Super()
{
	SampleRate = 60;
	GroundThreshold = 4.0f;
	SpeedThreshold = 0.1f;
	bShouldRemovePreExistingNotifiesOrSyncMarkers = false;
}

void UFootstepAnimEventsModifier::OnApply_Implementation(UAnimSequence* InAnimation)
{
	Super::OnApply_Implementation(InAnimation);

	if (InAnimation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "FootstepAnimEventsModifierBase failed. Reason: Invalid Animation");
		return;
	}
	
	// Disable root motion lock during application
	TGuardValue<bool> ForceRootLockGuard(InAnimation->bForceRootLock, false);
	
	// Clean up or generate tracks if needed
	ValidateNotifyTracks(InAnimation);
	
	// Process animation asset
	{
		const FAnimPoseEvaluationOptions AnimPoseEvalOptions { EAnimDataEvalType::Raw, true, false, false, nullptr, true, false };
		const float SequenceLength = InAnimation->GetPlayLength();
		const float SampleStep = 1.0f / static_cast<float>(SampleRate);
		const int SampleNum = FMath::TruncToInt(SequenceLength / SampleStep);
		
		TArray<FFootSampleState> FootSampleStates;
		FootSampleStates.Init(FFootSampleState(), FootDefinitions.Num());
		
		// Get ground levels and max speed values.
		for (int SampleIndex = 0; SampleIndex < SampleNum; ++SampleIndex)
		{
			const float SampleTime = FMath::Clamp(static_cast<float>(SampleIndex) * SampleStep, 0.0f, SequenceLength);
			const float FutureSampleTime = FMath::Clamp((static_cast<float>(SampleIndex) + 1.0f) * SampleStep, 0.0f, SequenceLength);

			FAnimPose AnimPose;
			FAnimPose FutureAnimPose;
			
			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, SampleTime, AnimPoseEvalOptions, AnimPose);
			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, FutureSampleTime, AnimPoseEvalOptions, FutureAnimPose);
			
			for (int FootIndex = 0; FootIndex < FootDefinitions.Num(); ++FootIndex)
			{
				const FFootDefinition & FootDef = FootDefinitions[FootIndex];
				FFootSampleState & FootState = FootSampleStates[FootIndex];
				
				FootState.GroundLevel = FMath::Min(FootState.GroundLevel, UAnimPoseExtensions::GetBonePose(AnimPose, FootDef.FootBoneName, EAnimPoseSpaces::World).GetLocation().Z);

				const float FootBoneSpeed = FMath::Abs(ComputeBoneSpeed(AnimPose, FutureAnimPose, SampleStep, FootDef.FootBoneName));
				FootState.MaxFootSpeed = FMath::Max(FootState.MaxFootSpeed, FootBoneSpeed);
				FootState.MinFootSpeed = FMath::Min(FootState.MinFootSpeed, FootBoneSpeed);
			}
		}

		// Generate animation events
		for (int SampleIndex = 0; SampleIndex < (SampleNum - 1); ++SampleIndex)
		{
			const float SampleTime = FMath::Clamp(static_cast<float>(SampleIndex) * SampleStep, 0.0f, SequenceLength);
			const float FutureSampleTime = FMath::Clamp((static_cast<float>(SampleIndex) + 1.0f) * SampleStep, 0.0f, SequenceLength);

			FAnimPose AnimPose;
			FAnimPose FutureAnimPose;
			
			// Query animation pose at the current sample time
			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, SampleTime, AnimPoseEvalOptions, AnimPose);
			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, FutureSampleTime, AnimPoseEvalOptions, FutureAnimPose);
			
			// Process all foot definitions
			for (int CurrFootIdx = 0; CurrFootIdx < FootDefinitions.Num(); ++CurrFootIdx)
			{
				const FFootDefinition & FootDef = FootDefinitions[CurrFootIdx];
				FFootSampleState & FootState = FootSampleStates[CurrFootIdx];
				
				// Update current sample info
				{
					const FTransform FootBoneTransform = UAnimPoseExtensions::GetBonePose(AnimPose, FootDef.FootBoneName, EAnimPoseSpaces::World);

					// Method: PassThroughReferenceBone
					{
						// Get reference bone translation
						const FTransform ReferenceBoneTransform = UAnimPoseExtensions::GetBonePose(AnimPose, FootDef.ReferenceBoneName, EAnimPoseSpaces::World);
						const FTransform FutureReferenceBoneTransform = UAnimPoseExtensions::GetBonePose(FutureAnimPose, FootDef.ReferenceBoneName, EAnimPoseSpaces::World);
						const FVector ReferenceBoneTranslation = FutureReferenceBoneTransform.GetLocation() - ReferenceBoneTransform.GetLocation();

						// Get foot bone direction with respect to reference bone
						const FVector ReferenceBoneToFootBoneVector = FootBoneTransform.GetLocation() - ReferenceBoneTransform.GetLocation();
					
						FootState.RefBoneTranslationDotRefBoneToFootBoneVec = FVector::DotProduct(ReferenceBoneTranslation, ReferenceBoneToFootBoneVector);
					}

					// Method: FootBoneReachesGround
					FootState.bIsFootBoneInGround = FMath::Abs(FootState.GroundLevel - FootBoneTransform.GetLocation().Z) <= GroundThreshold;

					// Method: FootBoneSpeed
					{
						// Note that the stored foot bone speed is being mapped from range (MinFootSpeed, MaxFootSpeed) to (0, 1).
						FootState.FootBoneSpeed = (FMath::Abs(ComputeBoneSpeed(AnimPose, FutureAnimPose, SampleStep, FootDef.FootBoneName)) - FootState.MinFootSpeed) / (FootState.MaxFootSpeed - FootState.MinFootSpeed);

						// Keep track of sample with lowest speed below threshold
						if (FootState.FootBoneSpeed < SpeedThreshold && SampleIndex > 1)
						{
							if (FootState.FootBoneSpeed < FootState.MinFootSpeedBelowThreshold && FMath::Abs(FootState.MinFootSpeedBelowThreshold - FootState.FootBoneSpeed) >= 0.01f)
							{
								FootState.TimeAtMinFootSpeedBelowThreshold = SampleTime;
								FootState.MinFootSpeedBelowThreshold = FootState.FootBoneSpeed;
							}
						}
					}
					
					// Keep track if we are processing the last sample
					FootState.bIsLast = SampleIndex == (SampleNum - 2);
				}

				// Place events when detection method passes
				if (SampleIndex > 1)
				{
					// Generate sync markers
					if (FootDef.bShouldGenerateSyncMarkers && CanWePlaceEventAtSample(FootState, FootDef.SyncMarkerDetectionTechnique))
					{
						if (!FootState.bSkipNextSyncMarker)
						{
							const float FinalSyncMarkerTime = FootDef.SyncMarkerDetectionTechnique == EDetectionTechnique::FootBoneSpeed ? FootState.TimeAtMinFootSpeedBelowThreshold : SampleTime;
							UAnimationBlueprintLibrary::AddAnimationSyncMarker(InAnimation, FootDef.SyncMarkerName, FinalSyncMarkerTime, FootDef.SyncMarkerTrackName);
						}
						else
						{
							FootState.bSkipNextSyncMarker = false;
						}
					}

					// Generate foot step fx notifies
					if (FootDef.bShouldGenerateNotifies && CanWePlaceEventAtSample(FootState, FootDef.FootstepNotifyDetectionTechnique))
					{
						if (!FootState.bSkipNextNotify)
						{
							const float FinalAnimNotifyTime = FootDef.FootstepNotifyDetectionTechnique == EDetectionTechnique::FootBoneSpeed ? FootState.TimeAtMinFootSpeedBelowThreshold : SampleTime;
							UAnimationBlueprintLibrary::AddAnimationNotifyEvent(InAnimation, FootDef.FootstepNotifyTrackName, FinalAnimNotifyTime, FootDef.FootstepNotify);
						}
						else
						{
							FootState.bSkipNextNotify = false;
						}
					}
				}
				
				// Update foot state
				{
					if (SampleIndex > 1)
					{
						const bool bDidEventPlacementTestPassUsingFootBoneSpeed = CanWePlaceEventAtSample(FootState, EDetectionTechnique::FootBoneSpeed);

						// Reset minimum values if we are above speed threshold
						if (bDidEventPlacementTestPassUsingFootBoneSpeed)
						{
							// Reset minimum values if we are above speed threshold
							FootState.TimeAtMinFootSpeedBelowThreshold = MAX_FLT;
							FootState.MinFootSpeedBelowThreshold = MAX_FLT;
						}
					}
					else
					{
						// Skip anim events if foot started below speed threshold
						FootState.bSkipNextNotify = FootState.FootBoneSpeed < SpeedThreshold && FootDef.bShouldSkipNotifyIfFootBoneSpeedStartsBelowThreshold;
						FootState.bSkipNextSyncMarker = FootState.FootBoneSpeed < SpeedThreshold && FootDef.bShouldSkipSyncMarkerIfFootBoneSpeedStartsBelowThreshold;
					}
					
					// Keep track of foot info
					FootState.PrevFootBoneSpeed = FootState.FootBoneSpeed;
					FootState.bWasFootBoneInGround = FootState.bIsFootBoneInGround;
					FootState.PrevRefBoneTranslationDotRefBoneToFootBoneVec = FootState.RefBoneTranslationDotRefBoneToFootBoneVec;
				}
			}
		}
	}
}

void UFootstepAnimEventsModifier::OnRevert_Implementation(UAnimSequence* InAnimation)
{
	Super::OnRevert_Implementation(InAnimation);

	// Delete any generate tracks.
	for (const FName GeneratedNotifyTrack : GeneratedNotifyTracks)
	{
		UAnimationBlueprintLibrary::RemoveAnimationNotifyTrack(InAnimation, GeneratedNotifyTrack);
	}
	
	// Delete any generated anim events.
	for (const FName ProcessedNotifyTrack : ProcessedNotifyTracks)
	{
		const bool IsNotifyTrackValid = UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(InAnimation, ProcessedNotifyTrack);

		if (IsNotifyTrackValid)
		{
			for (const FFootDefinition& FootDef : FootDefinitions)
			{
				if (FootDef.bShouldGenerateSyncMarkers && FootDef.SyncMarkerTrackName == ProcessedNotifyTrack)
				{
					UAnimationBlueprintLibrary::RemoveAnimationSyncMarkersByName(InAnimation, FootDef.SyncMarkerName);
				}

				if (FootDef.bShouldGenerateNotifies && FootDef.FootstepNotifyTrackName == ProcessedNotifyTrack)
				{
					if (FootDef.FootstepNotify)
					{
						UAnimationBlueprintLibrary::RemoveAnimationNotifyEventsByName(InAnimation, FName(Cast<UAnimNotify>(FootDef.FootstepNotify->GetDefaultObject())->GetNotifyName()));
					}
					else
					{
						UE_LOGF(LogAnimation, Error, "FootstepAnimEventsModifierBase failed. Reason: Invalid UAnimNotify class");
					}
				}
			}
		}
	}
	
	GeneratedNotifyTracks.Reset();
	ProcessedNotifyTracks.Reset();
}

void UFootstepAnimEventsModifier::ValidateNotifyTracks(UAnimSequence* InAnimation)
{
	check(InAnimation != nullptr)

	GatherNotifyTracksInfo(InAnimation);
	
	for (const FFootDefinition & FootDef : FootDefinitions)
	{
		if (FootDef.bShouldGenerateSyncMarkers)
		{
			PrepareNotifyTrack(InAnimation, FootDef.SyncMarkerTrackName);
		}
		
		if (FootDef.bShouldGenerateNotifies)
		{
			PrepareNotifyTrack(InAnimation, FootDef.FootstepNotifyTrackName);
		}
	}
}

void UFootstepAnimEventsModifier::GatherNotifyTracksInfo(const UAnimSequence* InAnimation)
{
	for (const FFootDefinition& FootDef : FootDefinitions)
	{
		// Determine tracks that will be generated and/or processed
		const bool bDoesRequestedSyncTrackAlreadyExist = UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(InAnimation, FootDef.SyncMarkerTrackName);
		const bool bDoesRequestedNotifyTrackAlreadyExist = UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(InAnimation, FootDef.FootstepNotifyTrackName);

		if (FootDef.bShouldGenerateSyncMarkers)
		{
			if (!bDoesRequestedSyncTrackAlreadyExist)
			{
				GeneratedNotifyTracks.Add(FootDef.SyncMarkerTrackName);
			}

			ProcessedNotifyTracks.Add(FootDef.SyncMarkerTrackName);
		}

		if (FootDef.bShouldGenerateNotifies)
		{
			if (!bDoesRequestedNotifyTrackAlreadyExist)
			{
				GeneratedNotifyTracks.Add(FootDef.FootstepNotifyTrackName);
			}

			ProcessedNotifyTracks.Add(FootDef.FootstepNotifyTrackName);
		}
	}
}

void UFootstepAnimEventsModifier::PrepareNotifyTrack(UAnimSequence* InAnimation, FName InRequestedNotifyTrackName)
{
	const bool bDoesTrackNameAlreadyExist = UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(InAnimation, InRequestedNotifyTrackName);
	
	if (!bDoesTrackNameAlreadyExist)
	{
		UAnimationBlueprintLibrary::AddAnimationNotifyTrack(InAnimation, InRequestedNotifyTrackName, FLinearColor::MakeRandomColor());
	}
	else if (bShouldRemovePreExistingNotifiesOrSyncMarkers)
	{
		UAnimationBlueprintLibrary::RemoveAnimationNotifyEventsByTrack(InAnimation, InRequestedNotifyTrackName);
		UAnimationBlueprintLibrary::RemoveAnimationSyncMarkersByTrack(InAnimation, InRequestedNotifyTrackName);
	}
}

bool UFootstepAnimEventsModifier::CanWePlaceEventAtSample(const FFootSampleState& InFootSampleState, EDetectionTechnique DetectionTechnique) const
{
	switch(DetectionTechnique)
	{
		case EDetectionTechnique::PassThroughReferenceBone: return InFootSampleState.RefBoneTranslationDotRefBoneToFootBoneVec > 0.0f && InFootSampleState.PrevRefBoneTranslationDotRefBoneToFootBoneVec < 0.0f;
		case EDetectionTechnique::FootBoneReachesGround: return !InFootSampleState.bWasFootBoneInGround && InFootSampleState.bIsFootBoneInGround;
		case EDetectionTechnique::FootBoneSpeed: return (InFootSampleState.PrevFootBoneSpeed < SpeedThreshold && InFootSampleState.FootBoneSpeed >= SpeedThreshold) || (InFootSampleState.bIsLast && InFootSampleState.FootBoneSpeed < SpeedThreshold && !FMath::IsNearlyEqual(InFootSampleState.MinFootSpeedBelowThreshold, MAX_FLT));
		default: return false;
	}
}

float UFootstepAnimEventsModifier::ComputeBoneSpeed(const FAnimPose& InPose, const FAnimPose& InFuturePose, float InDelta, FName InFootBoneName) const
{
	check(!FMath::IsNearlyZero(InDelta))
			
	const FTransform FootBoneTransform = UAnimPoseExtensions::GetBonePose(InPose, InFootBoneName, EAnimPoseSpaces::World);
	const FTransform FutureFootBoneTransform = UAnimPoseExtensions::GetBonePose(InFuturePose, InFootBoneName, EAnimPoseSpaces::World);
	const double FootBoneDistance = (FutureFootBoneTransform.GetLocation() - FootBoneTransform.GetLocation()).Length();
			
	return static_cast<float>(FootBoneDistance / InDelta);
}
#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
