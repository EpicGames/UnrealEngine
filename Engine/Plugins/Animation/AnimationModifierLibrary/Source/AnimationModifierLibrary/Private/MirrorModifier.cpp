// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirrorModifier.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimSequence.h"
#include "AnimationRuntime.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MirrorModifier)

#define LOCTEXT_NAMESPACE "MirrorModifier"

UMirrorModifier::UMirrorModifier()
	:Super()
{
}

void UMirrorModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "MirrorModifier failed. Reason: Invalid Animation");
		return;
	}

	IAnimationDataController& Controller = Animation->GetController();
	const IAnimationDataModel* Model = Animation->GetDataModel();

	if (Model == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "MirrorModifier failed. Reason: Invalid Data Model. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	if (!MirrorDataTable)
	{
		UE_LOGF(LogAnimation, Error, "MirrorModifier failed. Reason: Invalid Mirror Data table");
		return;
	}

	// Compact pose format of Mirror Bone Map
	TCustomBoneIndexArray<FCompactPoseBoneIndex, FCompactPoseBoneIndex> CompactPoseMirrorBones;
	FMemMark Mark(FMemStack::Get());

	// Pre-calculated component space of reference pose, which allows mirror to work with any joint orient 
	TCustomBoneIndexArray<FQuat, FCompactPoseBoneIndex> ComponentSpaceRefRotations;
	FBoneContainer BoneContainer;
	InitializeBoneContainer(Animation, BoneContainer);
	MirrorDataTable->FillCompactPoseAndComponentRefRotations(
		BoneContainer,
		CompactPoseMirrorBones,
		ComponentSpaceRefRotations);

	// Start editing animation data
	constexpr bool bShouldTransact = false;	Controller.OpenBracket(LOCTEXT("MirrorModifier_Bracket", "Updating bones"), bShouldTransact);
	const int32 NumKeys = Model->GetNumberOfKeys();

	TArray<FAnimPose>  StoredPoses;
	StoredPoses.Reserve(NumKeys);

	for (int32 AnimKey = 0; AnimKey < NumKeys; AnimKey++)
	{
		FAnimPose AnimPose;
		UAnimPoseExtensions::GetAnimPoseAtFrame(Animation, AnimKey, FAnimPoseEvaluationOptions(), AnimPose);
		StoredPoses.Emplace(MoveTemp(AnimPose));
	}
	
	for (int32 AnimKey = 0; AnimKey < NumKeys; AnimKey++)
	{
		FAnimPose& AnimPose = StoredPoses[AnimKey];
		FMemMark KeyMark(FMemStack::Get());
		FCompactPose OutPose;
		OutPose.SetBoneContainer(&BoneContainer);
		UAnimPoseExtensions::GetCompactPose(AnimPose, OutPose); 

		FAnimationRuntime::MirrorPose(OutPose, MirrorDataTable->MirrorAxis, CompactPoseMirrorBones, ComponentSpaceRefRotations);

		for (int32 BoneIndex = 0; BoneIndex < BoneContainer.GetNumBones(); ++BoneIndex)
		{
			FName SourceBoneName = BoneContainer.GetReferenceSkeleton().GetBoneName(BoneIndex);
			FSkeletonPoseBoneIndex SkeletonBoneIndex(BoneIndex);
			FCompactPoseBoneIndex CompactIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonBoneIndex);
			FTransform BonePoseTargetLocal = OutPose[CompactIndex];
			const FInt32Range KeyRangeToSet(AnimKey, AnimKey + 1);
			if (Model->IsValidBoneTrackName(SourceBoneName))
			{
				Controller.UpdateBoneTrackKeys(SourceBoneName, KeyRangeToSet, { BonePoseTargetLocal.GetLocation() }, { BonePoseTargetLocal.GetRotation() }, { BonePoseTargetLocal.GetScale3D() });
			}

		}	
	}

	if (bUpdateSyncMarkers)
	{
		for(FAnimSyncMarker& Marker : Animation->AuthoredSyncMarkers)
		{
			const FName* MirroredName = MirrorDataTable->SyncToMirrorSyncMap.Find(Marker.MarkerName);
			if (MirroredName)
			{
				Marker.MarkerName = *MirroredName;
			}
		}
		Animation->RefreshSyncMarkerDataFromAuthored(); 
	}

	if (bUpdateNotifies)
	{
		for(FAnimNotifyEvent& NotifyEvent : Animation->Notifies)
		{
			const FName* MirroredName = MirrorDataTable->AnimNotifyToMirrorAnimNotifyMap.Find(NotifyEvent.NotifyName);
			if (MirroredName)
			{
				NotifyEvent.NotifyName = *MirroredName;
			}
		}
		Animation->RefreshCacheData();
	}

	// Done editing animation data
	Controller.CloseBracket(bShouldTransact);
}

void UMirrorModifier::OnRevert_Implementation(UAnimSequence* Animation)
{
	//  mirroring will undo the previous mirror 
	OnApply_Implementation(Animation);
}

void UMirrorModifier::InitializeBoneContainer(const UAnimSequenceBase* AnimationSequenceBase, FBoneContainer& OutContainer) const
{
	// asset to use for retarget proportions (can be either USkeletalMesh or USkeleton)
	UObject* AssetToUse = CastChecked<UObject>(AnimationSequenceBase->GetSkeleton());
	int32 NumRequiredBones = AnimationSequenceBase->GetSkeleton()->GetReferenceSkeleton().GetNum();

	TArray<FBoneIndexType> RequiredBoneIndexArray;
	RequiredBoneIndexArray.AddUninitialized(NumRequiredBones);
	for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
	{
		RequiredBoneIndexArray[BoneIndex] = static_cast<FBoneIndexType>(BoneIndex);
	}

	OutContainer.InitializeTo(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *AssetToUse);
	OutContainer.SetUseRAWData(true);
	OutContainer.SetUseSourceData(true);
	OutContainer.SetDisableRetargeting(false);
}

#undef LOCTEXT_NAMESPACE
