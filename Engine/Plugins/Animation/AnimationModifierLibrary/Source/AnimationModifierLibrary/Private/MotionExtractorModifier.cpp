// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionExtractorModifier.h"
#include "Animation/Skeleton.h"
#include "MotionExtractorUtilities.h"
#include "Animation/AnimSequence.h"
#include "AnimationBlueprintLibrary.h"
#include "EngineLogs.h"
#include "MotionExtractorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionExtractorModifier)

#define LOCTEXT_NAMESPACE "UMotionExtractorModifier"

UMotionExtractorModifier::UMotionExtractorModifier()
	:Super()
{
	BoneName = FName(TEXT("root"));
	MotionType = EMotionExtractor_MotionType::Translation;
	Axis = EMotionExtractor_Axis::Y;
	bAbsoluteValue = false;
	bRelativeToFirstFrame = false;
	bComponentSpace_DEPRECATED = true;
	bNormalize = false;
	bUseCustomCurveName = false;
	CustomCurveName = NAME_None;
	SampleRate = 30;
	bRemoveCurveOnRevert = false;
}

void UMotionExtractorModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "MotionExtractorModifier failed. Reason: Invalid Animation");
		return;
	}

	USkeleton* Skeleton = Animation->GetSkeleton();
	if (Skeleton == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "MotionExtractorModifier failed. Reason: Animation with invalid Skeleton. Animation: %ls",
			*GetNameSafe(Animation));
		return;
	}

	const int32 BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogAnimation, Error, "MotionExtractorModifier failed. Reason: Invalid Bone Index. BoneName: %ls Animation: %ls Skeleton: %ls",
			*BoneName.ToString(), *GetNameSafe(Animation), *GetNameSafe(Skeleton));
		return;
	}

	int32 RelativeToBoneIndex = INDEX_NONE;
	const bool bUseRelativeToBone = Space == EMotionExtractor_Space::RelativeToBone;
	if (bUseRelativeToBone)
	{
		RelativeToBoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(RelativeToBoneName);
		if (RelativeToBoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogAnimation, Error, "MotionExtractorModifier failed. Reason: Invalid Bone Index for RelativeToBoneName. BoneName: %ls Animation: %ls Skeleton: %ls",
				*BoneName.ToString(), *GetNameSafe(Animation), *GetNameSafe(Skeleton));
			return;
		}
	}

	// When Space == EMotionExtractor_Space::RelativeToBone, operations only make sense to do in component space.
	const bool bShouldUseComponentSpace = Space != EMotionExtractor_Space::LocalSpace;

	// Ideally we would disable these options when any of those motion types are selected but AnimModifier doesn't support Details Customization atm.
	if((MotionType == EMotionExtractor_MotionType::Translation || MotionType == EMotionExtractor_MotionType::Rotation || MotionType == EMotionExtractor_MotionType::Scale) && Axis > EMotionExtractor_Axis::Z)
	{
		UE_LOGF(LogAnimation, Error, "MotionExtractorModifier failed. Reason: Only X, Y or Z axes are valid options for the selected motion type");
		return;
	}

	FMemMark Mark(FMemStack::Get());

	TGuardValue<bool> ForceRootLockGuard(Animation->bForceRootLock, false);

	TArray<FBoneIndexType> RequiredBones;
	RequiredBones.Add(IntCastChecked<FBoneIndexType>(BoneIndex));

	if (bUseRelativeToBone)
	{
		RequiredBones.Add(IntCastChecked<FBoneIndexType>(RelativeToBoneIndex));
	}

	Skeleton->GetReferenceSkeleton().EnsureParentsExistAndSort(RequiredBones);
	FBoneContainer BoneContainer(RequiredBones, UE::Anim::ECurveFilterMode::DisallowAll, *Skeleton);
	const FCompactPoseBoneIndex CompactPoseBoneIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(BoneIndex));

	FCompactPoseBoneIndex CompactPoseRelativeToBoneIndex = FCompactPoseBoneIndex(INDEX_NONE);
	FTransform FirstFrameRelativeToBoneTransform = FTransform::Identity;
	if (bUseRelativeToBone)
	{
		CompactPoseRelativeToBoneIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(RelativeToBoneIndex));
	}

	// If relative to first frame, use RelativeToBone's first frame as the reference frame.
	FTransform FirstFrameBoneTransform = UMotionExtractorUtilityLibrary::ExtractBoneTransform(Animation, BoneContainer, bUseRelativeToBone ? CompactPoseRelativeToBoneIndex : CompactPoseBoneIndex, 0.f, bShouldUseComponentSpace);

	const float AnimLength = Animation->GetPlayLength();
	const float SampleInterval = 1.f / static_cast<float>(SampleRate);

	FTransform LastBoneTransform = FTransform::Identity;
	float Time = 0.f;
	int32 SampleIndex = 0;

	float MaxValue = -MAX_FLT;
	if(bNormalize)
	{
		while (Time < AnimLength)
		{
			Time = FMath::Clamp(static_cast<float>(SampleIndex) * SampleInterval, 0.f, AnimLength);
			SampleIndex++;

			FTransform BoneTransform = UMotionExtractorUtilityLibrary::ExtractBoneTransform(Animation, BoneContainer, CompactPoseBoneIndex, Time, bShouldUseComponentSpace);

			if(bRelativeToFirstFrame)
			{
				BoneTransform = BoneTransform.GetRelativeTransform(FirstFrameBoneTransform);
			}
			else if (bUseRelativeToBone)
			{
				const FTransform RelativeToBoneTransform = UMotionExtractorUtilityLibrary::ExtractBoneTransform(Animation, BoneContainer, CompactPoseRelativeToBoneIndex, Time, bShouldUseComponentSpace);
				BoneTransform = BoneTransform.GetRelativeTransform(RelativeToBoneTransform);
			}

			// Ignore first frame if we are extracting something that depends on the previous bone transform
			if (SampleIndex > 1 || (MotionType != EMotionExtractor_MotionType::TranslationSpeed && MotionType != EMotionExtractor_MotionType::RotationSpeed))
			{
				const float Value = GetDesiredValue(BoneTransform, LastBoneTransform, SampleInterval);
				MaxValue = FMath::Max(FMath::Abs(Value), MaxValue);
			}

			LastBoneTransform = BoneTransform;
		}
	}
	
	LastBoneTransform = FTransform::Identity;
	Time = 0.f;
	SampleIndex = 0;

	TArray<FRichCurveKey> CurveKeys;
	
	while (Time < AnimLength)
	{
		Time = FMath::Clamp(static_cast<float>(SampleIndex) * SampleInterval, 0.f, AnimLength);
		SampleIndex++;

		FTransform BoneTransform = UMotionExtractorUtilityLibrary::ExtractBoneTransform(Animation, BoneContainer, CompactPoseBoneIndex, Time, bShouldUseComponentSpace);

		if(bRelativeToFirstFrame)
		{
			BoneTransform = BoneTransform.GetRelativeTransform(FirstFrameBoneTransform);
		}
		else if (bUseRelativeToBone)
		{
			const FTransform RelativeToBoneTransform = UMotionExtractorUtilityLibrary::ExtractBoneTransform(Animation, BoneContainer, CompactPoseRelativeToBoneIndex, Time, bShouldUseComponentSpace);
			BoneTransform = BoneTransform.GetRelativeTransform(RelativeToBoneTransform);
		}

		// Ignore first frame if we are extracting something that depends on the previous bone transform
		if (SampleIndex > 1 || (MotionType != EMotionExtractor_MotionType::TranslationSpeed && MotionType != EMotionExtractor_MotionType::RotationSpeed))
		{
			const float Value = GetDesiredValue(BoneTransform, LastBoneTransform, SampleInterval);
			const float FinalValue = (bNormalize && MaxValue != 0.f) ? FMath::Abs(Value) / MaxValue : Value;

			FRichCurveKey& Key = CurveKeys.AddDefaulted_GetRef();
			Key.Time = Time;
			Key.Value = FinalValue;
		}

		LastBoneTransform = BoneTransform;
	}

	IAnimationDataController& Controller = Animation->GetController();
	Controller.OpenBracket(LOCTEXT("SetMotionCurveBracket", "Setting Motion Curve"));
	const FAnimationCurveIdentifier CurveId = UAnimationCurveIdentifierExtensions::GetCurveIdentifier(Animation->GetSkeleton(), GetCurveName(), ERawCurveTrackTypes::RCT_Float);
	if(CurveKeys.Num() && Controller.AddCurve(CurveId))
	{
		Controller.SetCurveKeys(CurveId, CurveKeys);
	}
	Controller.CloseBracket();
}

void UMotionExtractorModifier::OnRevert_Implementation(UAnimSequence* Animation)
{
	if (bRemoveCurveOnRevert)
	{
		const FName FinalCurveName = GetCurveName();
		const bool bRemoveNameFromSkeleton = false;
		UAnimationBlueprintLibrary::RemoveCurve(Animation, FinalCurveName, bRemoveNameFromSkeleton);
	}
}

FName UMotionExtractorModifier::GetCurveName() const
{
	if (bUseCustomCurveName && !CustomCurveName.IsEqual(NAME_None))
	{
		return CustomCurveName;
	}

	return UMotionExtractorUtilityLibrary::GenerateCurveName(BoneName, MotionType, Axis);
}

float UMotionExtractorModifier::GetDesiredValue(const FTransform& BoneTransform, const FTransform& LastBoneTransform, float DeltaTime) const
{
	float Value = UMotionExtractorUtilityLibrary::GetDesiredValue(BoneTransform, LastBoneTransform, DeltaTime, MotionType, Axis);

	if (bAbsoluteValue)
	{
		Value = FMath::Abs(Value);
	}

	if (MathOperation != EMotionExtractor_MathOperation::None)
	{
		switch (MathOperation)
		{
		case EMotionExtractor_MathOperation::Addition:		Value = Value + Modifier; break;
		case EMotionExtractor_MathOperation::Subtraction:	Value = Value - Modifier; break;
		case EMotionExtractor_MathOperation::Division:		Value = Value / Modifier; break;
		case EMotionExtractor_MathOperation::Multiplication: Value = Value * Modifier; break;
		default: check(false); break;
		}
	}

	return Value;
}

void UMotionExtractorModifier::PostLoad()
{
	// If bComponentSpace wasn't its default value, set the space property and revert to default.
	if (bComponentSpace_DEPRECATED == false)
	{
		bComponentSpace_DEPRECATED = true;
		Space = EMotionExtractor_Space::LocalSpace;
	}

	Super::PostLoad();
}

#undef LOCTEXT_NAMESPACE