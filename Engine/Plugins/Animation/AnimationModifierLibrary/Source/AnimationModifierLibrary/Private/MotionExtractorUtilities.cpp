// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionExtractorUtilities.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AttributesRuntime.h"
#include "BonePose.h"
#include "EngineLogs.h"
#include "MotionExtractorTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionExtractorUtilities)

namespace UE::Anim::MotionExtractorUtility
{
	// Returns the ranges (X/Start to Y/End) in the specified animation sequence where the passes the specified criterion
	template<typename Predicate>
	TArray<FVector2D> GetRangesFromRootMotionByPredicate(const UAnimSequence* Animation, Predicate Pred, float SampleRate = 120.0f)
	{
		if (Animation == nullptr)
		{
			return TArray<FVector2D>();
		}

		const float AnimLength = Animation->GetPlayLength();
		float SampleInterval = 1.f / SampleRate;
		int32 NumSteps = static_cast<int32>(AnimLength / SampleInterval);

		TArray<FVector2D> OutRanges;
		OutRanges.Reserve(4);
		bool bMeetsCriterion = false;
		for (int32 Step = 0; Step < NumSteps; ++Step)
		{
			const double Time = static_cast<double>(Step) * SampleInterval;
			constexpr bool bAllowLooping = false;
			const FAnimExtractContext Context(Time, true, FDeltaTimeRecord(SampleInterval), bAllowLooping);
			const FTransform RootMotion = Animation->ExtractRootMotion(Context);
			const bool bMetCriterion = bMeetsCriterion;
			bMeetsCriterion = ::Invoke(Pred, RootMotion, SampleInterval);

			if (bMeetsCriterion != bMetCriterion)
			{
				if (bMeetsCriterion)
				{
					// Assume the range lasts from now until the animation ends.
					OutRanges.Add(FVector2D(Time, AnimLength));
				}
				else if (!OutRanges.IsEmpty())
				{
					// No longer meets the criterion. Close the range.
					OutRanges.Top().Y = Time;
				}
			}
		}

		return OutRanges;
	}
}

FName UMotionExtractorUtilityLibrary::GenerateCurveName(
	FName BoneName,
	EMotionExtractor_MotionType MotionType,
	EMotionExtractor_Axis Axis)
{
	FString MotionTypeStr;
	switch (MotionType)
	{
	case EMotionExtractor_MotionType::Translation:		MotionTypeStr = TEXT("translation"); break;
	case EMotionExtractor_MotionType::Rotation:			MotionTypeStr = TEXT("rotation"); break;
	case EMotionExtractor_MotionType::Scale:			MotionTypeStr = TEXT("scale"); break;
	case EMotionExtractor_MotionType::TranslationSpeed:	MotionTypeStr = TEXT("translation_speed"); break;
	case EMotionExtractor_MotionType::RotationSpeed:	MotionTypeStr = TEXT("rotation_speed"); break;
	default: check(false); break;
	}

	FString AxisStr;
	switch (Axis)
	{
	case EMotionExtractor_Axis::X:		AxisStr = TEXT("X"); break;
	case EMotionExtractor_Axis::Y:		AxisStr = TEXT("Y"); break;
	case EMotionExtractor_Axis::Z:		AxisStr = TEXT("Z"); break;
	case EMotionExtractor_Axis::XY:		AxisStr = TEXT("XY"); break;
	case EMotionExtractor_Axis::XZ:		AxisStr = TEXT("XZ"); break;
	case EMotionExtractor_Axis::YZ:		AxisStr = TEXT("YZ"); break;
	case EMotionExtractor_Axis::XYZ:	AxisStr = TEXT("XYZ"); break;
	default: check(false); break;
	}

	return FName(*FString::Printf(TEXT("%s_%s_%s"), *BoneName.ToString(), *MotionTypeStr, *AxisStr));
}

float UMotionExtractorUtilityLibrary::GetDesiredValue(
	const FTransform& BoneTransform, 
	const FTransform& LastBoneTransform, 
	float DeltaTime, 
	EMotionExtractor_MotionType MotionType,
	EMotionExtractor_Axis Axis)
{
	if ((MotionType == EMotionExtractor_MotionType::Translation || MotionType == EMotionExtractor_MotionType::Rotation || MotionType == EMotionExtractor_MotionType::Scale) && Axis > EMotionExtractor_Axis::Z)
	{
		UE_LOGF(LogAnimation, Error, "MotionExtractorUtilityLibrary failed. Reason: Only X, Y or Z axes are valid options for the selected motion type");
		return 0.0f;
	}

	double Value = 0.f;
	switch (MotionType)
	{
	case EMotionExtractor_MotionType::Translation:
	{
		const FVector Translation = BoneTransform.GetTranslation();
		switch (Axis)
		{
		case EMotionExtractor_Axis::X: Value = Translation.X; break;
		case EMotionExtractor_Axis::Y: Value = Translation.Y; break;
		case EMotionExtractor_Axis::Z: Value = Translation.Z; break;
		default: break;
		}
		break;
	}
	case EMotionExtractor_MotionType::Rotation:
	{
		const FRotator Rotation = BoneTransform.GetRotation().Rotator();
		switch (Axis)
		{
		case EMotionExtractor_Axis::X: Value = Rotation.Roll; break;
		case EMotionExtractor_Axis::Y: Value = Rotation.Pitch; break;
		case EMotionExtractor_Axis::Z: Value = Rotation.Yaw; break;
		default: break;
		}
		break;
	}
	case EMotionExtractor_MotionType::Scale:
	{
		const FVector Scale = BoneTransform.GetScale3D();
		switch (Axis)
		{
		case EMotionExtractor_Axis::X: Value = Scale.X; break;
		case EMotionExtractor_Axis::Y: Value = Scale.Y; break;
		case EMotionExtractor_Axis::Z: Value = Scale.Z; break;
		default: break;
		}
		break;
	}
	case EMotionExtractor_MotionType::TranslationSpeed:
	{
		if (!FMath::IsNearlyZero(DeltaTime))
		{
			const double Delta = UMotionExtractorUtilityLibrary::CalculateMagnitude((BoneTransform.GetTranslation() - LastBoneTransform.GetTranslation()), Axis);
			Value = Delta / DeltaTime;
		}

		break;
	}
	case EMotionExtractor_MotionType::RotationSpeed:
	{
		if (!FMath::IsNearlyZero(DeltaTime))
		{
			FQuat Delta = FQuat::Identity;
			FRotator Rotator = BoneTransform.GetRotation().Rotator();
			FRotator LastRotator = LastBoneTransform.GetRotation().Rotator();

			switch (Axis)
			{
			case EMotionExtractor_Axis::X:		Delta = FQuat(FRotator(0.f, 0.f, Rotator.Roll)) * FQuat(FRotator(0.f, 0.f, LastRotator.Roll)).Inverse(); break;
			case EMotionExtractor_Axis::Y:		Delta = FQuat(FRotator(Rotator.Pitch, 0.f, 0.f)) * FQuat(FRotator(LastRotator.Pitch, 0.f, 0.f)).Inverse(); break;
			case EMotionExtractor_Axis::Z:		Delta = FQuat(FRotator(0.f, Rotator.Yaw, 0.f)) * FQuat(FRotator(0.f, LastRotator.Yaw, 0.f)).Inverse(); break;
			case EMotionExtractor_Axis::XY:		Delta = FQuat(FRotator(Rotator.Pitch, 0.f, Rotator.Roll)) * FQuat(FRotator(LastRotator.Pitch, 0.f, LastRotator.Roll)).Inverse(); break;
			case EMotionExtractor_Axis::XZ:		Delta = FQuat(FRotator(0.f, Rotator.Yaw, Rotator.Roll)) * FQuat(FRotator(0.f, LastRotator.Yaw, LastRotator.Roll)).Inverse(); break;
			case EMotionExtractor_Axis::YZ:		Delta = FQuat(FRotator(Rotator.Pitch, Rotator.Yaw, 0.f)) * FQuat(FRotator(LastRotator.Pitch, LastRotator.Yaw, 0.f)).Inverse(); break;
			case EMotionExtractor_Axis::XYZ:	Delta = BoneTransform.GetRotation() * LastBoneTransform.GetRotation().Inverse(); break;
			default: break;
			}

			float RotationAngle = 0.f;
			FVector RotationAxis;
			Delta.ToAxisAndAngle(RotationAxis, RotationAngle);
			RotationAngle = FMath::UnwindRadians(RotationAngle);
			if (RotationAngle < 0.0f)
			{
				RotationAxis = -RotationAxis;
				RotationAngle = -RotationAngle;
			}

			RotationAngle = FMath::RadiansToDegrees(RotationAngle);
			Value = RotationAngle / DeltaTime;
		}

		break;
	}
	default: check(false); break;
	}

	return static_cast<float>(Value);
}

TArray<FVector2D> UMotionExtractorUtilityLibrary::GetStoppedRangesFromRootMotion(const UAnimSequence* AnimSequence, float StopSpeedThreshold, float SampleRate)
{
	auto Criterion = [StopSpeedThreshold](const FTransform& InRootMotionDelta, float DeltaTime)
	{
		const FVector RootMotionTranslation = InRootMotionDelta.GetTranslation();
		const double RootMotionSpeed = RootMotionTranslation.Size() / DeltaTime;
		return RootMotionSpeed < StopSpeedThreshold;
	};

	return UE::Anim::MotionExtractorUtility::GetRangesFromRootMotionByPredicate(AnimSequence, Criterion, SampleRate);
}

TArray<FVector2D> UMotionExtractorUtilityLibrary::GetMovingRangesFromRootMotion(const UAnimSequence* AnimSequence, float StopSpeedThreshold /*= 10.0f*/, float SampleRate /*= 120.0f*/)
{
	auto Criterion = [StopSpeedThreshold](const FTransform& InRootMotionDelta, float DeltaTime)
	{
		const FVector RootMotionTranslation = InRootMotionDelta.GetTranslation();
		const double RootMotionSpeed = RootMotionTranslation.Size() / DeltaTime;
		return RootMotionSpeed > StopSpeedThreshold;
	};

	return UE::Anim::MotionExtractorUtility::GetRangesFromRootMotionByPredicate(AnimSequence, Criterion, SampleRate);
}

FTransform UMotionExtractorUtilityLibrary::ExtractBoneTransform(UAnimSequence* Animation, const FBoneContainer& BoneContainer, FCompactPoseBoneIndex CompactPoseBoneIndex, float Time, bool bComponentSpace)
{
	FCompactPose Pose;
	Pose.SetBoneContainer(&BoneContainer);

	FBlendedCurve Curve;
	Curve.InitFrom(BoneContainer);

	FAnimExtractContext Context(static_cast<double>(Time), false);
	UE::Anim::FStackAttributeContainer Attributes;
	FAnimationPoseData AnimationPoseData(Pose, Curve, Attributes);

	Animation->GetBonePose(AnimationPoseData, Context, true);

	check(Pose.IsValidIndex(CompactPoseBoneIndex));

	if (bComponentSpace)
	{
		FCSPose<FCompactPose> ComponentSpacePose;
		ComponentSpacePose.InitPose(Pose);
		return ComponentSpacePose.GetComponentSpaceTransform(CompactPoseBoneIndex);
	}
	else
	{
		return Pose[CompactPoseBoneIndex];
	}
}

double UMotionExtractorUtilityLibrary::CalculateMagnitude(const FVector& Vector, EMotionExtractor_Axis Axis)
{
	switch (Axis)
	{
	case EMotionExtractor_Axis::X:		return FMath::Abs(Vector.X);
	case EMotionExtractor_Axis::Y:		return FMath::Abs(Vector.Y);
	case EMotionExtractor_Axis::Z:		return FMath::Abs(Vector.Z);
	case EMotionExtractor_Axis::XY:		return FMath::Sqrt(Vector.X * Vector.X + Vector.Y * Vector.Y);
	case EMotionExtractor_Axis::XZ:		return FMath::Sqrt(Vector.X * Vector.X + Vector.Z * Vector.Z);
	case EMotionExtractor_Axis::YZ:		return FMath::Sqrt(Vector.Y * Vector.Y + Vector.Z * Vector.Z);
	case EMotionExtractor_Axis::XYZ:	return FMath::Sqrt(Vector.X * Vector.X + Vector.Y * Vector.Y + Vector.Z * Vector.Z);
	default: check(false); break;
	}

	return 0.f;
}
