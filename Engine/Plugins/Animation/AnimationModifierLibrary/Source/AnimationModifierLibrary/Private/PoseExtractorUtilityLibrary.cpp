// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PoseExtractorUtilityLibrary.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AnimationRuntime.h"

FPoseExtractorUtilityLibrary::FPoseExtractorUtilityLibrary(UAnimSequence* Animation)
	: Skeleton(nullptr)
	, Model(nullptr)
{
	if (!Animation)
	{
		UE_LOGF(LogAnimation, Error, "FPoseExtractorUtilityLibrary failed. Reason: Invalid Animation");
		return;
	}

	IAnimationDataController& Controller = Animation->GetController();
	const IAnimationDataModel* AnimationModel = Animation->GetDataModel();

	if (!AnimationModel)
	{
		UE_LOGF(LogAnimation, Error, "FPoseExtractorUtilityLibrary failed. Reason: Invalid Data Model. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	USkeleton* AnimationSkeleton = Animation->GetSkeleton();
	if (!AnimationSkeleton)
	{
		UE_LOGF(LogAnimation, Error, "FPoseExtractorUtilityLibrary failed. Reason: Invalid Skeleton. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	const FReferenceSkeleton& RefSkeleton = AnimationSkeleton->GetReferenceSkeleton();
	if (RefSkeleton.GetNum() == 0)
	{
		UE_LOGF(LogAnimation, Error, "FPoseExtractorUtilityLibrary failed. Reason: Ref Skeleton. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	Skeleton = AnimationSkeleton;
	Model = AnimationModel;
}

bool FPoseExtractorUtilityLibrary::AddBone(const FBoneReference& Bone)
{
	if (!Skeleton)
	{
		return false;
	}

	check(ReducedIndexMap.IsEmpty());

	FBoneReference TempBone = Bone;
	TempBone.Initialize(Skeleton);
	if (!TempBone.HasValidSetup())
	{
		UE_LOGF(LogAnimation, Error, "FPoseExtractorUtilityLibrary::AddBone failed. Reason: Bone %ls has invalid setup", *TempBone.BoneName.ToString());
		return false;
	}

	return AddBone(TempBone.BoneName, TempBone.BoneIndex);
}

bool FPoseExtractorUtilityLibrary::AddBone(const FName& BoneName, int32 BoneIndex)
{
	if (BoneIndex >= 0 && BoneIndex < MAX_uint16)
	{
		NameToBoneIndex.FindOrAdd(BoneName) = static_cast<FBoneIndexType>(BoneIndex);
		BoneIndicesWithParents.AddUnique(static_cast<FBoneIndexType>(BoneIndex));
		return true;
	}
	
	UE_LOGF(LogAnimation, Error, "FPoseExtractorUtilityLibrary::AddBone failed. Reason: Bone %ls has invalid BoneIndex %d", *BoneName.ToString(), BoneIndex);
	return false;
}

bool FPoseExtractorUtilityLibrary::CacheGlobalBoneTransforms()
{
	if (!Skeleton)
	{
		return false;
	}

	if (BoneIndicesWithParents.IsEmpty())
	{
		UE_LOGF(LogAnimation, Warning, "FPoseExtractorUtilityLibrary::CacheGlobalBoneTransforms no weighted bones?");
		return false;
	}

	BoneIndicesWithParents.Sort();

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	check(RefSkeleton.GetNum() > 0);

	FAnimationRuntime::EnsureParentsPresent(BoneIndicesWithParents, RefSkeleton);

	for (int32 i = 0; i < BoneIndicesWithParents.Num(); ++i)
	{
		ReducedIndexMap.Add(BoneIndicesWithParents[i]) = IntCastChecked<FBoneIndexType>(i);
	}
	
	const int32 NumAnimKeys = Model->GetNumberOfKeys();
	GlobalBoneTransforms.SetNum(NumAnimKeys);
	for (int32 AnimKey = 0; AnimKey < NumAnimKeys; AnimKey++)
	{
		GlobalBoneTransforms[AnimKey].SetNum(BoneIndicesWithParents.Num());

		// calculating all the required global transforms
		for (int i = 0; i < BoneIndicesWithParents.Num(); ++i)
		{
			const uint16 BoneIndex = BoneIndicesWithParents[i];
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
			const FTransform LocalBoneTransform = Model->EvaluateBoneTrackTransform(BoneName, AnimKey, EAnimInterpolationType::Step);
			const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(BoneIndex);
			if (ParentBoneIndex != INDEX_NONE)
			{
				const uint16* MappedParentBoneIndex = ReducedIndexMap.Find(IntCastChecked<FBoneIndexType>(ParentBoneIndex));
				check(MappedParentBoneIndex);
				check(*MappedParentBoneIndex <= i);
				GlobalBoneTransforms[AnimKey][i] = LocalBoneTransform * GlobalBoneTransforms[AnimKey][*MappedParentBoneIndex];
			}
			else
			{
				GlobalBoneTransforms[AnimKey][i] = LocalBoneTransform;
			}
		}
	}

	return true;
}

const FTransform& FPoseExtractorUtilityLibrary::GetGlobalBoneTransform(int32 AnimKey, uint16 BoneIndex) const
{
	const uint16* MappedParentBoneIndex = ReducedIndexMap.Find(BoneIndex);
	check(MappedParentBoneIndex);
	return GlobalBoneTransforms[AnimKey][*MappedParentBoneIndex];
}

const FTransform& FPoseExtractorUtilityLibrary::GetGlobalBoneTransform(int32 AnimKey, const FName& BoneName) const
{
	const uint16* BoneIndex = NameToBoneIndex.Find(BoneName);
	check(BoneIndex);
	return GetGlobalBoneTransform(AnimKey, *BoneIndex);
}

#endif // WITH_EDITOR
