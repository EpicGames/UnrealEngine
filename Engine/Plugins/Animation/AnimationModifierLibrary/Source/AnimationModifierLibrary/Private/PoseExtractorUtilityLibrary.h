// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"

struct FBoneReference;
class IAnimationDataModel;
class UAnimSequence;
class USkeleton;

struct FPoseExtractorUtilityLibrary
{
	FPoseExtractorUtilityLibrary(UAnimSequence* Animation);
	bool AddBone(const FBoneReference& Bone);
	bool AddBone(const FName& BoneName, int32 BoneIndex);
	bool CacheGlobalBoneTransforms();
	const FTransform& GetGlobalBoneTransform(int32 AnimKey, uint16 BoneIndex) const;
	const FTransform& GetGlobalBoneTransform(int32 AnimKey, const FName& BoneName) const;
	
private:
	const USkeleton* Skeleton = nullptr;
	const IAnimationDataModel* Model = nullptr;
	TArray<uint16> BoneIndicesWithParents;
	TMap<uint16, uint16> ReducedIndexMap;
	TMap<FName, uint16> NameToBoneIndex;
	TArray<TArray<FTransform>> GlobalBoneTransforms;
};

#endif // WITH_EDITOR
