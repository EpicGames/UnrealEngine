// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReOrientRootBoneModifier.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "AnimationUtils.h"
#include "Animation/Skeleton.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReOrientRootBoneModifier)

#define LOCTEXT_NAMESPACE "ReOrientRootBoneModifier"

UReOrientRootBoneModifier::UReOrientRootBoneModifier()
	:Super()
{
}

void UReOrientRootBoneModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	ReOrientRootBone_Internal(Animation, Rotator.Quaternion());
}

void UReOrientRootBoneModifier::OnRevert_Implementation(UAnimSequence* Animation)
{
	ReOrientRootBone_Internal(Animation, Rotator.Quaternion().Inverse());
}

void UReOrientRootBoneModifier::ReOrientRootBone_Internal(UAnimSequence* Animation, const FQuat& Quat)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "ReOrientMeshModifier failed. Reason: Invalid Animation");
		return;
	}

	IAnimationDataController& Controller = Animation->GetController();
	const IAnimationDataModel* Model = Animation->GetDataModel();

	if (Model == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "ReOrientMeshModifier failed. Reason: Invalid Data Model. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	const USkeleton* Skeleton = Animation->GetSkeleton();
	if (Skeleton == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "ReOrientMeshModifier failed. Reason: Invalid Skeleton. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	if (RefSkeleton.GetNum() == 0)
	{
		UE_LOGF(LogAnimation, Error, "ReOrientMeshModifier failed. Reason: Ref Skeleton. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	// Collect tracks for the bones we are going to modify (root and direct children of the root)
	const FName RootBoneName = RefSkeleton.GetBoneName(0);
	TArray<FName> RootBoneChildBones;

	TArray<int32> ChildBoneIndices;
	RefSkeleton.GetDirectChildBones(0, ChildBoneIndices);

	for (const int32& ChildBoneIndex : ChildBoneIndices)
	{
		const FName ChildBoneName = RefSkeleton.GetBoneName(ChildBoneIndex);
		if (Model->IsValidBoneTrackName(ChildBoneName))
		{
			RootBoneChildBones.Add(ChildBoneName);
		}
	}
	
	// Start editing animation data
	const bool bShouldTransact = false;
	Controller.OpenBracket(LOCTEXT("ReorientingRootBone_Bracket", "Reorienting root bone"), bShouldTransact);

	struct FBoneKeys
	{
		TArray<FVector> Location;
		TArray<FQuat> Rotation;
		TArray<FVector> Scale;

		void Add(const FTransform& Transform)
		{
			Location.Add(Transform.GetLocation());
			Rotation.Add(Transform.GetRotation());
			Scale.Add(Transform.GetScale3D());
		}		
	};

	FBoneKeys NewRootKeys;
	TMap<FName, FBoneKeys> NewChildKeys;
		
	// For each key in the animation
	const int32 Num = Model->GetNumberOfKeys();
	for (int32 AnimKey = 0; AnimKey < Num; AnimKey++)
	{
		const FInt32Range KeyRangeToSet(AnimKey, AnimKey + 1);

		// Reorient the root bone
		const FTransform RootTransformOriginal = Model->EvaluateBoneTrackTransform(RootBoneName, AnimKey, EAnimInterpolationType::Step);

		FTransform RootTransformNew = RootTransformOriginal;
		RootTransformNew.SetRotation(RootTransformOriginal.GetRotation() * Quat);
		NewRootKeys.Add(RootTransformNew);
		
		// Now the mesh is facing the wrong axis. Update direct children of the root with the local space transform that puts them back to where they were originally
		for (const FName& ChildBoneName : RootBoneChildBones)
		{
			FBoneKeys& ChildKeys = NewChildKeys.FindOrAdd(ChildBoneName);
			
			FTransform OtherBoneTransform = Model->EvaluateBoneTrackTransform(ChildBoneName, AnimKey, EAnimInterpolationType::Step);
			OtherBoneTransform = OtherBoneTransform * RootTransformOriginal;
			OtherBoneTransform = OtherBoneTransform.GetRelativeTransform(RootTransformNew);
			ChildKeys.Add(OtherBoneTransform);
		}
	}

	Controller.SetBoneTrackKeys(RootBoneName, NewRootKeys.Location, NewRootKeys.Rotation, NewRootKeys.Scale);
	
	for (const FName& ChildBoneName : RootBoneChildBones)
	{
		const FBoneKeys& ChildKeys = NewChildKeys.FindChecked(ChildBoneName);
		Controller.SetBoneTrackKeys(ChildBoneName, ChildKeys.Location, ChildKeys.Rotation, ChildKeys.Scale);
	}

	// Done editing animation data
	Controller.CloseBracket(bShouldTransact);
}

#undef LOCTEXT_NAMESPACE
