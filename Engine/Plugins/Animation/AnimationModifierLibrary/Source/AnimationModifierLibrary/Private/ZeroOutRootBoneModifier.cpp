// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZeroOutRootBoneModifier.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimSequenceHelpers.h"
#include "AnimationUtils.h"
#include "Animation/Skeleton.h"
#include "EngineLogs.h"
#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZeroOutRootBoneModifier)

#define LOCTEXT_NAMESPACE "ZeroOutRootBoneModifier"

UZeroOutRootBoneModifier::UZeroOutRootBoneModifier()
	:Super()
{
}

void UZeroOutRootBoneModifier::OnApply_Implementation(UAnimSequence* Animation)
{
	if (Animation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "ZeroOutRootBoneModifier failed. Reason: Invalid Animation");
		return;
	}

	IAnimationDataController& Controller = Animation->GetController();
	const IAnimationDataModel* Model = Animation->GetDataModel();

	if (Model == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "ZeroOutRootBoneModifier failed. Reason: Invalid Data Model. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	const USkeleton* Skeleton = Animation->GetSkeleton();
	if (Skeleton == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "ZeroOutRootBoneModifier failed. Reason: Invalid Skeleton. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	if (RefSkeleton.GetNum() == 0)
	{
		UE_LOGF(LogAnimation, Error, "ZeroOutRootBoneModifier failed. Reason: Ref Skeleton. Animation: %ls", *GetNameSafe(Animation));
		return;
	}

	const FName RootBoneName = RefSkeleton.GetBoneName(0);

	// Start editing animation data
	const bool bShouldTransact = false;
	Controller.OpenBracket(LOCTEXT("ZeroOutRootBoneModifier_Bracket", "Zero Out Root Bone"), bShouldTransact);

	const int32 NumKeys = Model->GetNumberOfKeys();

	// Cache root transforms and find where motion starts/ends.
	TArray<FTransform> PreModifiedRootTransforms;
	PreModifiedRootTransforms.Reserve(NumKeys);
	int32 KeyWhereMotionStarts = -1;
	int32 KeyWhereMotionEnds = 0;
	FScopedSlowTask SlowTask(static_cast<float>(NumKeys));
	SlowTask.MakeDialog();
	for (int32 AnimKey = 0; AnimKey < NumKeys; AnimKey++)
	{
		SlowTask.EnterProgressFrame();
		const FTransform RootTransform = Model->EvaluateBoneTrackTransform(RootBoneName, AnimKey, EAnimInterpolationType::Step);
		PreModifiedRootTransforms.Add(RootTransform);

		if (AnimKey + 1 < NumKeys)
		{
			const FTransform NextRootTransform = Model->EvaluateBoneTrackTransform(RootBoneName, AnimKey + 1, EAnimInterpolationType::Step);
			const bool bRootTransformChanged = !FTransform::AreTranslationsEqual(RootTransform, NextRootTransform) || !FTransform::AreRotationsEqual(RootTransform, NextRootTransform);

			if (bRootTransformChanged)
			{
				if (KeyWhereMotionStarts < 0)
				{
					KeyWhereMotionStarts = AnimKey;
				}

				KeyWhereMotionEnds = AnimKey + 1;
			}
		}
	}

	// For each key in the animation
	for (int32 AnimKey = 0; AnimKey < NumKeys; AnimKey++)
	{
		const FInt32Range KeyRangeToSet(AnimKey, AnimKey + 1);

		FTransform NewRootTransform = PreModifiedRootTransforms[AnimKey] * PreModifiedRootTransforms[0].Inverse();
		Controller.UpdateBoneTrackKeys(RootBoneName, KeyRangeToSet, { NewRootTransform.GetLocation() }, { NewRootTransform.GetRotation() }, { NewRootTransform.GetScale3D() }, bShouldTransact);
	}

	// Done editing animation data
	Controller.CloseBracket(bShouldTransact);
	
	if (bClipEndFramesWithNoMotion && (KeyWhereMotionEnds > KeyWhereMotionStarts) && (KeyWhereMotionEnds + 1 < NumKeys))
	{
		// Remove everything after the frame where motion ends.
		const int32 FirstKeyToRemove = KeyWhereMotionEnds + 1;
		const int32 NumKeysToRemove = NumKeys - FirstKeyToRemove;
		UE::Anim::AnimationData::RemoveKeys(Animation, FirstKeyToRemove, NumKeysToRemove);
	}

	if (bClipStartFramesWithNoMotion && (KeyWhereMotionStarts > 1))
	{
		// Remove everything before the frame where motion starts.
		const int32 NumKeysToRemove = KeyWhereMotionStarts - 1;
		UE::Anim::AnimationData::RemoveKeys(Animation, 0, NumKeysToRemove);
	}
}

#undef LOCTEXT_NAMESPACE
