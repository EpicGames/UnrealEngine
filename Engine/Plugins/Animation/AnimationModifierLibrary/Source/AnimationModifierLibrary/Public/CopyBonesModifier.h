// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "AnimPose.h"
#include "BoneContainer.h"
#include "CopyBonesModifier.generated.h"

USTRUCT(BlueprintType)
struct FBoneReferencePair
{
	GENERATED_BODY()

	/** Bone to get transform from  */
	UPROPERTY(EditAnywhere, Category = "Default")
	FBoneReference SourceBone;

	/** Bone to update with the transform copied from SourceBone */
	UPROPERTY(EditAnywhere, Category = "Default")
	FBoneReference TargetBone;
};

/** Animation Modifier to copy the transform of 'SourceBone(s)' to 'TargetBone(s)' */
UCLASS()
class UCopyBonesModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	/** Source and Target bone pairs */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	TArray<FBoneReferencePair> BonePairs;

	/** Space to convert SourceBone transforms into before copying them to TargetBone */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	EAnimPoseSpaces BonePoseSpace = EAnimPoseSpaces::World;

	UCopyBonesModifier();

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
	virtual void OnRevert_Implementation(UAnimSequence* Animation) override;

};
