// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "CurveFromSyncMarkersModifier.generated.h"

/** A sync marker name and the float value to assign to the curve at that marker's time */
USTRUCT(Experimental, BlueprintType)
struct FSyncMarkerCurveEntry
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FName SyncMarkerName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	float Value = 0.f;
};

/** Generates a float curve from the authored sync markers on an animation sequence.
 *  Each sync marker listed in SyncMarkerValues contributes a curve key at the marker's time
 *  with the corresponding float value. */
UCLASS(Experimental)
class UCurveFromSyncMarkersModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:
	/** List of sync marker names and the float value each should contribute to the generated curve */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FSyncMarkerCurveEntry> SyncMarkerValues;

	/** Name of the float curve to add to the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FName CurveName = "CurveFromSyncMarkersModifier";

	/** If true, the generated curve is removed when the modifier is reverted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bRemoveCurveOnRevert = true;

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
	virtual void OnRevert_Implementation(UAnimSequence* Animation) override;
};
