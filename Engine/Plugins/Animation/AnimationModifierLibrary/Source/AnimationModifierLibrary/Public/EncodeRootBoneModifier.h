// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "AnimationModifier.h"
#include "EncodeRootBoneModifier.generated.h"

UENUM()
enum class EEncodeRootBoneAxis : uint8
{
	X,
	Y,
	Z,
};

USTRUCT()
struct FEncodeRootBoneWeightedBone
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference Bone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	float Weight = 1.f;
};

USTRUCT()
struct FEncodeRootBoneWeightedBoneAxis : public FEncodeRootBoneWeightedBone
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	EEncodeRootBoneAxis BoneAxis = EEncodeRootBoneAxis::Y;
};

UCLASS()
class UEncodeRootBoneModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FEncodeRootBoneWeightedBone> WeightedBoneToComputeRootPosition;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FEncodeRootBoneWeightedBoneAxis> WeightedBoneToComputeRootOrientation;

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
};

#endif // WITH_EDITOR