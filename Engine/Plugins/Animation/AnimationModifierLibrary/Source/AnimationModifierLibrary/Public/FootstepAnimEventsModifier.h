// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "AnimPose.h"
#include "FootstepAnimEventsModifier.generated.h"

class UAnimNotify;

/**  Detection method used for placing a notify / marker in the track */
UENUM()
enum class EDetectionTechnique : uint8
{
	/**
	 * Create anim event when foot bone passes through a given reference bone.
	 *
	 * Note that the translation vector of the reference bone will be used as the normal vector for the detection plane
	 * used to determine if the foot bone has crossed the reference bone. */
	PassThroughReferenceBone,
	
	/**
	 * Create anim event when foot bone reaches the ground level within a given threshold.
	 * 
	 * Note that this will only be true if the foot bone position goes from NOT being within the threshold to being
	 * WITHIN the ground threshold.
	 */
	FootBoneReachesGround,

	/**
	 * Create anim event when foot bone translation speed is below a given threshold and nearly zero.
	 *
	 * Note that the foot bone translation speed is normalize therefore when a footstep occurs
	 * the speed will be very close to zero.
	 */
	FootBoneSpeed
};

USTRUCT(BlueprintType)
struct FFootDefinition
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Default")
	FName FootBoneName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Default")
	FName ReferenceBoneName = FName(TEXT("root"));
	
	UPROPERTY(EditAnywhere, Category = "Sync Markers")
	bool bShouldGenerateSyncMarkers = false; 
	
	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta=(EditCondition="bShouldGenerateSyncMarkers"))
	FName SyncMarkerTrackName = TEXT("FootSyncMarkers");

	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta=(EditCondition="bShouldGenerateSyncMarkers"))
	FName SyncMarkerName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta=(EditCondition="bShouldGenerateSyncMarkers"))
	EDetectionTechnique SyncMarkerDetectionTechnique = EDetectionTechnique::PassThroughReferenceBone;

	UPROPERTY(EditAnywhere, Category = "Sync Markers", meta=(EditCondition = "SyncMarkerDetectionTechnique == EDetectionTechnique::FootBoneSpeed", EditConditionHides))
	bool bShouldSkipSyncMarkerIfFootBoneSpeedStartsBelowThreshold = true;
	
	UPROPERTY(EditAnywhere, Category = "Notifies")
	bool bShouldGenerateNotifies = false;
	
	UPROPERTY(EditAnywhere, Category = "Notifies", meta=(EditCondition="bShouldGenerateNotifies"))
	FName FootstepNotifyTrackName = TEXT("FootAnimEvents");

	UPROPERTY(EditAnywhere, Category = "Notifies", meta=(EditCondition="bShouldGenerateNotifies"))
	TSubclassOf<UAnimNotify> FootstepNotify;

	UPROPERTY(EditAnywhere, Category = "Notifies", meta=(EditCondition="bShouldGenerateNotifies"))
	EDetectionTechnique FootstepNotifyDetectionTechnique = EDetectionTechnique::FootBoneSpeed;

	UPROPERTY(EditAnywhere, Category = "Notifies", meta=(EditCondition = "FootstepNotifyDetectionTechnique == EDetectionTechnique::FootBoneSpeed", EditConditionHides))
	bool bShouldSkipNotifyIfFootBoneSpeedStartsBelowThreshold = true;
};

/** Generates animation notifies and/or sync markers for any specified bone(s) */
UCLASS(meta=(IsBlueprintBase=true))
class UFootstepAnimEventsModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	UFootstepAnimEventsModifier();
	
	/** Rate used to sample the animation */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings", meta = (Units="Hz", UIMin=1))
	int SampleRate;

	/** Threshold for determining if a foot bone position can be considered to be on the ground level */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	float GroundThreshold;

	/**
	 * Threshold to start finding the smallest foot bone translation speed.
	 *
	 * Note that the foot bone translation speed is normalize therefore when a footstep occurs
	 * the speed will be very close to zero, thus for most cases this value won't need to be changed.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings", meta = (ClampMin=0.0f, ClampMax=1.0f))
	float SpeedThreshold;
	
	/** Foot bone(s) to be processed */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings", meta = (TitleProperty = "FootBoneName"))
	TArray<FFootDefinition> FootDefinitions;
	
	/**
	 * If true, applying the anim modifier becomes a destructive action, meaning that any existing matched tracks will have their data overwritten by the modifier.
	 * Otherwise, no previous notifies or sync markers will removed when applying the anim modifier.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings", DisplayName="Remove Pre Existing Notifies or Sync Markers")
	bool bShouldRemovePreExistingNotifiesOrSyncMarkers;

	/** Begin UAnimationModifier interface */
	virtual void OnApply_Implementation(UAnimSequence* InAnimation) override;
	virtual void OnRevert_Implementation(UAnimSequence* InAnimation) override;
	/** End UAnimationModifier interface */

private:

	/** Store state of a foot during a sample step */
	struct FFootSampleState
	{
		double GroundLevel = UE_DOUBLE_BIG_NUMBER;
		double RefBoneTranslationDotRefBoneToFootBoneVec = 0.0;
		double PrevRefBoneTranslationDotRefBoneToFootBoneVec = 0.0;
		bool bIsFootBoneInGround = false;
		bool bWasFootBoneInGround = false; 
		float FootBoneSpeed = 0.0f;
		float PrevFootBoneSpeed = 0.0f;
		float MaxFootSpeed = -MAX_FLT;
		float MinFootSpeed = MAX_FLT;
		float MinFootSpeedBelowThreshold = MAX_FLT;
		float TimeAtMinFootSpeedBelowThreshold = MAX_FLT;
		bool bIsLast = false;
		bool bSkipNextSyncMarker = false;
		bool bSkipNextNotify = false;
	};
	
	/** Keep track of to be generated tracks during modifier application */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Settings"))
	TSet<FName> GeneratedNotifyTracks;

	/** Keep track of tracks modified during modifier application */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Settings"))
	TSet<FName> ProcessedNotifyTracks;
	
	/** Prepare all requested notify tracks before starting to generate notifies or sync markers */
	void ValidateNotifyTracks(UAnimSequence* InAnimation);

	/** Determine which notify tracks will be generated when this modifier is applied */
	void GatherNotifyTracksInfo(const UAnimSequence* InAnimation);

	/** Create new notify track if requested track does not exist, otherwise, if the requested track exist, clean-up data if needed. */
	void PrepareNotifyTrack(UAnimSequence* InAnimation, FName InRequestedNotifyTrackName);

	/** Test if we can place anim event at given sample */
	bool CanWePlaceEventAtSample(const FFootSampleState & InTestLegSampleState, EDetectionTechnique DetectionTechnique) const;

	/** Get a bone translation speed */
	float ComputeBoneSpeed(const FAnimPose& InPose, const FAnimPose &InFuturePose, float InDelta, FName InFootBoneName) const;
};
