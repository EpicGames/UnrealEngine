// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "AnimPose.h"
#include "BoneContainer.h"
#include "Animation/MirrorDataTable.h"
#include "MirrorModifier.generated.h"


/** Animation Modifier to mirror an animation using a mirror data table */
UCLASS()
class UMirrorModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	//MirrorDataTable that will be used for mirroring the aniamtion
	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UMirrorDataTable> MirrorDataTable;

	// If sync markers should be mirrored in the animation 
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool  bUpdateSyncMarkers = true;

	// If notifies should be mirrored in the animation 
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool  bUpdateNotifies = true;
	
	UMirrorModifier();
	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
	virtual void OnRevert_Implementation(UAnimSequence* Animation) override;
protected:
	void InitializeBoneContainer(const UAnimSequenceBase* AnimationSequenceBase, FBoneContainer& OutContainer) const;
};
