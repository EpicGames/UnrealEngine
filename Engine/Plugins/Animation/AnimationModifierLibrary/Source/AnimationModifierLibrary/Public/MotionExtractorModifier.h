// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "MotionExtractorModifier.generated.h"

enum class EMotionExtractor_Axis : uint8;
enum class EMotionExtractor_MathOperation : uint8;
enum class EMotionExtractor_MotionType : uint8;
enum class EMotionExtractor_Space : uint8;

/** Extracts motion from a bone in the animation and bakes it into a curve */
UCLASS()
class UMotionExtractorModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	/** Bone we are going to generate the curve from */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FName BoneName;

	/** Bone which we'll use as the reference frame to generate this curve from. Space must be RelativeToBone to use this. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(EditCondition="Space == EMotionExtractor_Space::RelativeToBone"))
	FName RelativeToBoneName;

	/** Type of motion to extract */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EMotionExtractor_MotionType MotionType;

	/** Axis to get the value from */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EMotionExtractor_Axis Axis;

	/** Whether we want to remove the curve when we revert or re-apply modifier 
		Disabling this allows you to modify settings and create a new curve each time you re-apply the modifier
		Enabling this is the preferred setting when using a modifier that's applied in bulk and you may want to remove/rename curves */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bRemoveCurveOnRevert;

	/** Whether to extract the bone transforms relative to the first frame in the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bRelativeToFirstFrame;

	/** Reference frame/space to use when extracting the bone pose. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EMotionExtractor_Space Space;

	/** Whether to extract the bone pose in component space or local space. */
	UPROPERTY()
	bool bComponentSpace_DEPRECATED;

	/** Whether to convert the final value to absolute (positive) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bAbsoluteValue;

	/** Optional math operation to apply on the extracted value before add it to the generated curve */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	EMotionExtractor_MathOperation MathOperation;

	/** Right operand for the math operation selected */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "MathOperation != EMotionExtractor_MathOperation::None"))
	float Modifier;

	/** Whether we want a normalized value (0 - 1) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bNormalize;

	/** Rate used to sample the animation */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "1"))
	int32 SampleRate;

	/** Whether we want to specify a custom name for the curve. If false, the name of the curve will be auto generated based on the data we are going to extract */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bUseCustomCurveName;

	/** Custom name for the curve we are going to generate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta = (EditCondition = "bUseCustomCurveName"))
	FName CustomCurveName;

	UMotionExtractorModifier();

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
	virtual void OnRevert_Implementation(UAnimSequence* Animation) override;

	/** Returns the name for the curve. If CurveName is None it generates a name based on the data we are going to extract */
	FName GetCurveName() const;

	/** Returns the desired value from the extracted poses */
	float GetDesiredValue(const FTransform& BoneTransform, const FTransform& LastBoneTransform, float DeltaTime) const;

	virtual void PostLoad() override;
};
