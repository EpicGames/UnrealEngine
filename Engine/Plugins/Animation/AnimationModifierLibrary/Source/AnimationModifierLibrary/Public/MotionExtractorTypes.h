// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"
#include "Misc/EnumRange.h"
#include "MotionExtractorTypes.generated.h"

/** Type of motion to extract */
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EMotionExtractor_MotionType : uint8
{
	None				= 0 UMETA(Hidden),
	Translation			= 1 << 0,
	Rotation			= 1 << 1,
	Scale				= 1 << 2,
	TranslationSpeed	= 1 << 3,
	RotationSpeed		= 1 << 4,
};
ENUM_CLASS_FLAGS(EMotionExtractor_MotionType);
constexpr bool EnumHasAnyFlags(int32 Flags, EMotionExtractor_MotionType Contains) { return (Flags & int32(Contains)) != 0; }
ENUM_RANGE_BY_VALUES(EMotionExtractor_MotionType, 
	EMotionExtractor_MotionType::Translation, 
	EMotionExtractor_MotionType::Rotation,
	EMotionExtractor_MotionType::Scale,
	EMotionExtractor_MotionType::TranslationSpeed,
	EMotionExtractor_MotionType::RotationSpeed);

/** Axis to get the final value from */
UENUM(BlueprintType)
enum class EMotionExtractor_Axis : uint8
{
	X,
	Y,
	Z,
	XY,
	XZ,
	YZ,
	XYZ
};

/** Reference frame/space to use when calculating motion */
UENUM(BlueprintType)
enum class EMotionExtractor_Space : uint8
{
	ComponentSpace,
	LocalSpace,
	RelativeToBone,
};

/** Math operations that can be applied to the extracted value before add it to the curve */
UENUM(BlueprintType)
enum class EMotionExtractor_MathOperation : uint8
{
	None,
	Addition,
	Subtraction,
	Division,
	Multiplication
};
