// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/Package.h"
#include "MotionExtractorUtilities.generated.h"

#define UE_API ANIMATIONMODIFIERLIBRARY_API

class UAnimSequence;
enum class EMotionExtractor_Axis : uint8;
enum class EMotionExtractor_MotionType : uint8;
struct FBoneContainer;
struct FCompactPoseBoneIndex;

UCLASS(MinimalAPI, meta=(ScriptName="MotionExtractorUtilityLibrary"))
class UMotionExtractorUtilityLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	* Generates a curve name based on input settings.
	*
	* @param BoneName			   The name of the bone
	* @param MotionType            What type of motion this curve represents (translation, rotation, speed, etc.)
	* @param Axis				   Which axis/axes the motion for this curve was extracted from
	*/
	UFUNCTION(BlueprintCallable, Category = "Motion Extractor Utility")
	static UE_API FName GenerateCurveName(
		FName BoneName,
		EMotionExtractor_MotionType MotionType,
		EMotionExtractor_Axis Axis);

	/**
	* Returns the desired value from the extracted poses based on input settings.
	*
	* @param BoneTransform         Current frame's bone transform
	* @param LastBoneTransform     Last frame's bone transform. Unused when not calculating speeds.
	* @param DeltaTime			   Time step used between current and last bone transforms. Unused when not calculating speeds.
	* @param MotionType            What type of motion to extract (translation, rotation, speed, etc.)
	* @param Axis				   Which axis/axes to extract motion from
	*/
	UFUNCTION(BlueprintCallable, Category = "Motion Extractor Utility")
	static UE_API float GetDesiredValue(
		const FTransform& BoneTransform,
		const FTransform& LastBoneTransform,
		float DeltaTime,
		EMotionExtractor_MotionType MotionType,
		EMotionExtractor_Axis Axis);

	/**
	* Returns the ranges (X/Start to Y/End) in the specified animation sequence where the animation is considered stopped.
	*
	* @param AnimSequence			Anim sequence to check
	* @param StopSpeedThreshold		Root motion speed under which the animation is considered stopped.
	* @param SampleRate				Sample rate of the animation. It's recommended to use high values if the animation has very sudden direction changes.
	*/
	UFUNCTION(BlueprintCallable, Category = "Motion Extractor Utility")
	static UE_API TArray<FVector2D> GetStoppedRangesFromRootMotion(const UAnimSequence* AnimSequence, float StopSpeedThreshold = 10.0f, float SampleRate = 120.0f);

	/**
	* Returns the ranges (X/Start to Y/End) in the specified animation sequence where the animation is considered moving.
	*
	* @param AnimSequence			Anim sequence to check
	* @param StopSpeedThreshold		Root motion speed over which the animation is considered moving.
	* @param SampleRate				Sample rate of the animation. It's recommended to use high values if the animation has very sudden direction changes.
	*/
	UFUNCTION(BlueprintCallable, Category = "Motion Extractor Utility")
	static UE_API TArray<FVector2D> GetMovingRangesFromRootMotion(const UAnimSequence* AnimSequence, float StopSpeedThreshold = 10.0f, float SampleRate = 120.0f);

	/** 
	* Helper function to extract the pose for a given bone at a given time
	* IMPORTANT: This function expects you to add a MemMark (FMemMark Mark(FMemStack::Get());) at the correct scope if you are using it from outside world's tick
	*/
	static UE_API FTransform ExtractBoneTransform(UAnimSequence* Animation, const FBoneContainer& BoneContainer, FCompactPoseBoneIndex CompactPoseBoneIndex, float Time, bool bComponentSpace);

	/** Helper function to calculate the magnitude of a vector only considering a specific axis or axes */
	static UE_API double CalculateMagnitude(const FVector& Vector, EMotionExtractor_Axis Axis);
};

#undef UE_API
