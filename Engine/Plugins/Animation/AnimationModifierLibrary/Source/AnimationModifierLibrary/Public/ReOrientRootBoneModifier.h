// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "ReOrientRootBoneModifier.generated.h"

/** Reorient root bone in the animation while maintaining mesh position and rotation */
UCLASS()
class UReOrientRootBoneModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	/** Rotation to apply to the root */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	FRotator Rotator;

	UReOrientRootBoneModifier();

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;
	virtual void OnRevert_Implementation(UAnimSequence* Animation) override;

private:

	void ReOrientRootBone_Internal(UAnimSequence* Animation, const FQuat& Quat);

};
