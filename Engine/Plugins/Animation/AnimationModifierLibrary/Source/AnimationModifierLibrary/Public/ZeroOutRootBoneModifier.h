// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "ZeroOutRootBoneModifier.generated.h"

/** Adjust root motion to be relative to the first frame. Optionally cut the start and end sections of the animation that don't have motion on the root.
	This was written to be used when capturing Character Movement motion via Take Recorder. Take Recorder outputs an animation captured from
	a character moving in game in world space, and this modifier zeroes out the root. The animation can then be exported to fbx to be animated against. */
UCLASS()
class UZeroOutRootBoneModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:

	UZeroOutRootBoneModifier();

	virtual void OnApply_Implementation(UAnimSequence* Animation) override;

	// Clip frames at the start of the animation that have no root motion.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	bool bClipStartFramesWithNoMotion = true;

	// Clip frames at the end of the animation that have no root motion.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings")
	bool bClipEndFramesWithNoMotion = true;
};